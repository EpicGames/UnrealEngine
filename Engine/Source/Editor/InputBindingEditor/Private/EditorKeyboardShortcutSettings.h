// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/DeveloperSettings.h"
#include "EditorKeyboardShortcutSettings.generated.h"

UCLASS(config=EditorKeyBindings, meta=(DisplayName="Keyboard Shortcuts"))
class UEditorKeyboardShortcutSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	virtual bool SupportsAutoRegistration() const override { return false;}
};


