// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorKeyboardShortcutSettings.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/InputBindingManager.h"
#include "Framework/Commands/UICommandInfo.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModule.h"
#include "ISettingsEditorModule.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Interfaces/IInputBindingEditorModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Logging/MessageLog.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "TimerManager.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UnrealEdMisc.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SChordEditBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;

#define LOCTEXT_NAMESPACE "InputBindingEditor"

namespace InputBindingEditorModule
{
static const FName SettingsModuleName("Settings");
static const FName PropertyEditorModuleName("PropertyEditor");

bool bShowBindingNames = false;
static FAutoConsoleVariableRef CVarDebugBindingNames(TEXT("Input.Debug.ShowBindingNames"), bShowBindingNames, TEXT("True to show binding names in the input binding editor."));
}

/**
 * A gesture sort functor.  Sorts by name or gesture and ascending or descending
 */
struct FChordSort
{
	FChordSort( bool bInSortName, bool bInSortUp )
		: bSortName( bInSortName )
		, bSortUp( bInSortUp )
	{ }

	bool operator()( const TSharedPtr<FUICommandInfo>& A, const TSharedPtr<FUICommandInfo>& B ) const
	{
		if( bSortName )
		{
			// Sort by command bundle, and then by command label. If a command has no bundle,
			// it will compare its label to the other command's bundle.
			const int32 CompareResult = GetPrimaryTextForCommand(A).CompareTo(GetPrimaryTextForCommand(B));
			bool bFinalResult = CompareResult < 0;
			if (CompareResult == 0)
			{
				bFinalResult = A->GetLabel().CompareTo(B->GetLabel()) < 0;
			}
			return bSortUp ? !bFinalResult : bFinalResult;
		}
		else
		{
			// Sort by binding
			bool bResult = A->GetInputText().CompareTo( B->GetInputText() ) < 0;
			return bSortUp ? !bResult : bResult;
		}
	}

private:
	/** Helper function to check if command is in a bundle, and if so, return the bundle description */
	const FText& GetPrimaryTextForCommand(const TSharedPtr<FUICommandInfo>& Command) const
	{
		if (Command->GetBundle() != NAME_None)
		{
			TSharedPtr<FBindingContext> Context = FInputBindingManager::Get().GetContextByName(Command->GetBindingContext());
			return Context->GetBundleLabel(Command->GetBundle());
		}
		else
		{
			return Command->GetLabel();
		}
	}

	/** Whether or not to sort by name.  If false we sort by binding. */
	bool bSortName;

	/** Whether or not to sort up.  If false we sort down. */
	bool bSortUp;
};

/**
* An item for the chord tree view
*/
struct FChordTreeItem
{
	// Note these are mutually exclusive
	TWeakPtr<FBindingContext> BindingContext;
	TSharedPtr<FUICommandInfo> CommandInfo;

	TSharedPtr<FBindingContext> GetBindingContext() { return BindingContext.Pin(); }

	bool IsContext() const { return BindingContext.IsValid(); }
	bool IsCommand() const { return CommandInfo.IsValid(); }
};

class FEditorKeyboardShortcutSettings : public IDetailCustomization
{
public:

	FEditorKeyboardShortcutSettings() :
		bUpdateRequested(false),
		DetailBuilder(nullptr)
	{
	}

	virtual ~FEditorKeyboardShortcutSettings()
	{
		FBindingContext::CommandsChanged.RemoveAll(this);
		FInputBindingManager::Get().SaveInputBindings();
	}

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable( new FEditorKeyboardShortcutSettings );
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override
	{
		DetailBuilder = &InDetailBuilder;

		UpdateContextList();
		UpdateUI();

		FBindingContext::CommandsChanged.AddSP( SharedThis( this ), &FEditorKeyboardShortcutSettings::OnCommandsChanged );
	}

	/** Updates the context list with new commands. */
	void UpdateContextList()
	{
		FInputBindingManager& InputBindingManager = FInputBindingManager::Get();

		TArray< TSharedPtr<FBindingContext> > Contexts;
		InputBindingManager.GetKnownInputContexts( Contexts );

		// Filter to allowed bindings
		Contexts.RemoveAll([&InputBindingManager](const TSharedPtr<FBindingContext>& Context)
		{
			return !InputBindingManager.CommandPassesFilter(FName(), Context->GetContextName());
		});

		Contexts.Sort([](const TSharedPtr<FBindingContext>& A, const TSharedPtr<FBindingContext>& B)
		{
			return A->GetContextDesc().CompareTo(B->GetContextDesc()) < 0;
		});

		/** List of all known contexts. */
		ContextList.Reset(Contexts.Num());

		for (const TSharedPtr<FBindingContext>& Context : Contexts)
		{
			TSharedRef<FChordTreeItem> TreeItem( new FChordTreeItem );
			TreeItem->BindingContext = Context;
			ContextList.Add( TreeItem );
		}
	}

	void ForceRefreshDetails()
	{
		bUpdateRequested = false;
		UpdateContextList();

		if (DetailBuilder)
		{
			FBindingContext::CommandsChanged.RemoveAll(this);

			IDetailLayoutBuilder* DetailBuilderPtr = DetailBuilder;
			DetailBuilder = nullptr;
			DetailBuilderPtr->ForceRefreshDetails();
		}
	}

	void OnCommandsChanged(const FBindingContext& ContextThatChanged)
	{
		if (!bUpdateRequested)
		{
			bUpdateRequested = true;
			GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &FEditorKeyboardShortcutSettings::ForceRefreshDetails));
		}
	}

	void UpdateUI()
	{
		FInputBindingManager& InputBindingManager = FInputBindingManager::Get();

		static const FText SearchKeyKeybindUnbound = LOCTEXT("SearchKeyKeybindUnbound", "Unbound");
		static const FText SearchKeyKeybind = LOCTEXT("SearchKeyKeybind", "KeyBind:");
		static const FText SearchKeyContext = LOCTEXT("SearchKeyContext", "Context:");
		static const FTextFormat TokenCombineFormat = INVTEXT("{0} {1}");
		static const FTextFormat TokenDelimiter = INVTEXT("\"{0}\""); // Use for strict search instead of partial...

		ISettingsEditorModule::FSearchTerm* KeybindSearchTerm = nullptr;
		ISettingsEditorModule::FSearchTerm* ContextSearchTerm = nullptr;
		if (ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>(TEXT("SettingsEditor")))
		{
			if (const TSharedPtr<IDetailsView> DetailsView = DetailBuilder->GetDetailsViewSharedPtr())
			{
				const FName SearchContext = DetailsView->GetIdentifier();
				
				// Register first
				const bool bKeybindRegistered = SettingsEditorModule->RegisterSearchTerm(SearchContext, SearchKeyKeybind.ToString());
				const bool bContextRegistered = SettingsEditorModule->RegisterSearchTerm(SearchContext, SearchKeyContext.ToString());

				if (bKeybindRegistered)
				{
					KeybindSearchTerm = SettingsEditorModule->FindSearchTerm(SearchContext, SearchKeyKeybind.ToString());
					KeybindSearchTerm->Label = LOCTEXT("SearchKeyKeybindLabel", "Keybind");
					KeybindSearchTerm->Tooltip = LOCTEXT("SearchKeyKeybindTooltip", "Filters by the pressed key combination");
					KeybindSearchTerm->IconStyleSetName = FAppStyle::GetAppStyleSetName();
					KeybindSearchTerm->IconStyleName = TEXT("GraphEditor.KeyEvent_16x");
				}

				if (bContextRegistered)
				{
					ContextSearchTerm = SettingsEditorModule->FindSearchTerm(SearchContext, SearchKeyContext.ToString());
					ContextSearchTerm->Label = LOCTEXT("SearchKeyContextLabel", "Context");
					ContextSearchTerm->Tooltip = LOCTEXT("SearchKeyContextTooltip", "Filters by the command context");
					// Hide this option from being shown in filter buttons but allow it to be suggested
					ContextSearchTerm->bShowFilter = false;
				}
			}
		}

		for (TSharedPtr<FChordTreeItem>& TreeItem : ContextList)
		{
			check(TreeItem->IsContext());

			FName BindingContextName = TreeItem->GetBindingContext()->GetContextName();
			const FText& BindingContextLabel = TreeItem->GetBindingContext()->GetContextDesc();

			IDetailCategoryBuilder& CategoryBuilder = DetailBuilder->EditCategory(BindingContextName, BindingContextLabel);

			TArray<TSharedPtr<FUICommandInfo>> Commands;
			InputBindingManager.GetCommandInfosFromContext(BindingContextName, Commands);

			// Filter to allowed bindings
			Commands.RemoveAll([&InputBindingManager](const TSharedPtr<FUICommandInfo>& CommandInfo)
			{
				return !InputBindingManager.CommandPassesFilter(CommandInfo->GetBindingContext(), CommandInfo->GetCommandName());
			});
			
			Commands.Sort(FChordSort(true, false));

			if (KeybindSearchTerm)
			{
				// Commands * 2 since there are primary and secondary keybinds
				KeybindSearchTerm->Values.Reserve(KeybindSearchTerm->Values.Num() + (Commands.Num() * 2));
			}

			TMap<FName, IDetailGroup*> BundleMap;
			for(TSharedPtr<FUICommandInfo>& CommandInfo : Commands)
			{
				FDetailWidgetRow* Row = nullptr;
				const FName BundleName = CommandInfo->GetBundle();
				if (!BundleName.IsNone())
				{
					if (!BundleMap.Contains(BundleName))
					{
						// TreeItem is guaranteed to have BindingContext due to check() above
						const FText& BundleLabel = TreeItem->GetBindingContext()->GetBundleLabel(BundleName);
						IDetailGroup* Group = BundleMap.Add(BundleName, &CategoryBuilder.AddGroup(BundleName, BundleLabel));
						// Match this widget with the "Label" widget on non-bundled commands (see below)
						Group->HeaderRow().NameContent()
						.MaxDesiredWidth(0)
						.MinDesiredWidth(500)
						[
							SNew(SBox)
							.Padding(FMargin(0.0f, 3.0f, 0.0f, 3.0f))
							[
								SNew(STextBlock)
								.Text(BundleLabel)
							]
						];
					}
					Row = &BundleMap[BundleName]->AddWidgetRow();
					Row->FilterString(CommandInfo->GetLabel());
				}
				else
				{
					Row = &CategoryBuilder.AddCustomRow(CommandInfo->GetLabel());
				}

				{
					// Set up search filter (for i.e. keybind: "F")
					const TSharedRef<const FInputChord> FirstInputChord = CommandInfo->GetActiveChord(EMultipleKeyBindingIndex::Primary);
					const TSharedRef<const FInputChord> SecondInputChord = CommandInfo->GetActiveChord(EMultipleKeyBindingIndex::Secondary);

					const bool bFirstChordValid = FirstInputChord->IsValidChord();
					const bool bSecondChordValid = SecondInputChord->IsValidChord();

					FText FirstKeyFilter, SecondKeyFilter;
					if (bFirstChordValid)
					{
						FirstKeyFilter = FText::FormatOrdered(TokenDelimiter, FirstInputChord->GetInputText());

						// Register search key and value for settings editor suggestions
						if (KeybindSearchTerm)
						{
							KeybindSearchTerm->Values.Add(FirstInputChord->GetInputText().ToString());
						}
					}

					if (bSecondChordValid)
					{
						SecondKeyFilter = FText::FormatOrdered(TokenDelimiter, SecondInputChord->GetInputText());

						// Register search key and value for settings editor suggestions
						if (KeybindSearchTerm)
						{
							KeybindSearchTerm->Values.Add(SecondInputChord->GetInputText().ToString());
						}
					}

					FText FilterText = bFirstChordValid && bSecondChordValid
						? FText::FormatOrdered(TokenCombineFormat, FirstKeyFilter, SecondKeyFilter)
						: (bFirstChordValid ? FirstKeyFilter : SecondKeyFilter);

					// Allow search for unbound commands like this : keybind: "unbound"
					if (FilterText.IsEmpty())
					{
						FilterText = FText::FormatOrdered(TokenDelimiter, SearchKeyKeybindUnbound.ToLower());

						// Register search key and value for settings editor suggestions
						if (KeybindSearchTerm)
						{
							KeybindSearchTerm->Values.Add(SearchKeyKeybindUnbound.ToString());
						}
					}

					// Command label and similar string might already be stored
					if (!Row->FilterTextString.IsEmpty())
					{
						FilterText = FText::FormatOrdered(TokenCombineFormat, Row->FilterTextString, FilterText);
					}

					// Append command binding context
					{
						FText BindingContext = FText::FormatOrdered(TokenDelimiter, FText::FromName(BindingContextName));

						// Register search key and value for settings editor suggestions
						if (ContextSearchTerm)
						{
							ContextSearchTerm->Values.Add(BindingContextName.ToString());
						}

						FilterText = FText::FormatOrdered(TokenCombineFormat, FilterText, BindingContext);
					}

					// Append keywords
					FilterText = FText::FormatOrdered(TokenCombineFormat, FilterText, SearchKeyKeybind.ToLower());
					FilterText = FText::FormatOrdered(TokenCombineFormat, FilterText, SearchKeyContext.ToLower());

					Row->FilterString(FilterText);
				}

				Row->NameContent()
				.MaxDesiredWidth(0)
				.MinDesiredWidth(500)
				[
					SNew(STextBlock)
					.Text(CommandInfo->GetLabel())
					.ToolTipText_Lambda([CommandInfo]() -> FText
					{
						FText CommandInfoTooltip = CommandInfo->GetDescription();

						if (InputBindingEditorModule::bShowBindingNames)
						{
							CommandInfoTooltip = FText::Format(LOCTEXT("CommandInfoDebugToolTip", "{0}\n\nBinding Context: {1}\nCommand Name: {2}"), CommandInfoTooltip, FText::FromName(CommandInfo->GetBindingContext()), FText::FromName(CommandInfo->GetCommandName()));
						}

						// Show default chords	
						const FInputChord& DefaultChord1 = CommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Primary);
						const FInputChord& DefaultChord2 = CommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Secondary);
						CommandInfoTooltip = FText::Format(
							INVTEXT("{0}\n\nDefault Primary: \"{1}\"\nDefault Secondary: \"{2}\""),
							CommandInfoTooltip,
							DefaultChord1.GetInputText(),
							DefaultChord2.GetInputText()
						);

						return CommandInfoTooltip;
					})
				];

				Row->ValueContent()
				.MaxDesiredWidth(200)
				.MinDesiredWidth(200)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1.0f, 0.0f, 9.0f, 0.0f)
					[
						SNew(SChordEditBox, CommandInfo, EMultipleKeyBindingIndex::Primary)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SChordEditBox, CommandInfo, EMultipleKeyBindingIndex::Secondary)
					]
				];
				
				Row->OverrideResetToDefault(
					FResetToDefaultOverride::Create(
						TAttribute<bool>::CreateSP(this, &FEditorKeyboardShortcutSettings::IsCommandResetToDefaultVisible, CommandInfo),
						FSimpleDelegate::CreateSP(this, &FEditorKeyboardShortcutSettings::ResetToDefaultCommand, CommandInfo)
					)
				);
			}
		}
	}

	bool IsCommandResetToDefaultVisible(TSharedPtr<FUICommandInfo> InCommandInfo) const
	{
		if (!InCommandInfo)
		{
			return false;
		}

		const FInputChord& DefaultChord1 = InCommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Primary);
		const TSharedRef<const FInputChord> ActiveChord1 = InCommandInfo->GetActiveChord(EMultipleKeyBindingIndex::Primary);
		if (ActiveChord1.Get() != DefaultChord1)
		{
			return true;
		}

		const FInputChord& DefaultChord2 = InCommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Secondary);
		const TSharedRef<const FInputChord> ActiveChord2 = InCommandInfo->GetActiveChord(EMultipleKeyBindingIndex::Secondary);
		if (ActiveChord2.Get() != DefaultChord2)
		{
			return true;
		}

		return false;
	}

	void ResetToDefaultCommand(TSharedPtr<FUICommandInfo> InCommandInfo)
	{
		if (!InCommandInfo)
		{
			return;
		}

		const FInputChord& DefaultChord1 = InCommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Primary);
		InCommandInfo->SetActiveChord(DefaultChord1, EMultipleKeyBindingIndex::Primary);

		const FInputChord& DefaultChord2 = InCommandInfo->GetDefaultChord(EMultipleKeyBindingIndex::Secondary);
		InCommandInfo->SetActiveChord(DefaultChord2, EMultipleKeyBindingIndex::Secondary);
	}

private:
	bool bUpdateRequested;
	IDetailLayoutBuilder* DetailBuilder;
	/** List of all known contexts. */
	TArray< TSharedPtr<FChordTreeItem> > ContextList;
};

class FInputBindingEditorModule
	: public IInputBindingEditorModule
{
public:

	// IInputBindingEditorModule interface
	virtual void StartupModule() override
	{
		ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>(InputBindingEditorModule::SettingsModuleName);

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(InputBindingEditorModule::PropertyEditorModuleName);

		EditorKeyboardShortcutSettingsName = UEditorKeyboardShortcutSettings::StaticClass()->GetFName();
		PropertyEditor.RegisterCustomClassLayout(EditorKeyboardShortcutSettingsName, FOnGetDetailCustomizationInstance::CreateStatic(&FEditorKeyboardShortcutSettings::MakeInstance));

		// input bindings
		const FName ContainerName = "InputBinding";
		const FName CategoryName = "Shortcuts";
		const ISettingsSectionPtr InputBindingSettingsSection = SettingsModule.RegisterSettings(
			ContainerName, 
			CategoryName, 
			"InputBindings",
			LOCTEXT("InputBindingsSettingsName", "Keyboard Shortcuts"),
			LOCTEXT("InputBindingsSettingsDescription", "Configure keyboard shortcuts to quickly invoke operations."),
			GetMutableDefault<UEditorKeyboardShortcutSettings>()
		);

		if (const TSharedPtr<ISettingsContainer> EditorContainer = SettingsModule.GetContainer(ContainerName))
		{
			// Place it below "All Settings"
			EditorContainer->SetCategorySortPriority(CategoryName, -1);
			EditorContainer->DescribeCategory(CategoryName, LOCTEXT("EditorShortcutsCategoryName", "Shortcuts"), LOCTEXT("EditorShortcutsCategoryDescription", "Shortcuts settings for the editor"));
		}

		if(InputBindingSettingsSection.IsValid())
		{
			InputBindingSettingsSection->OnExport().BindRaw(this, &FInputBindingEditorModule::HandleInputBindingsExport);
			InputBindingSettingsSection->OnImport().BindRaw(this, &FInputBindingEditorModule::HandleInputBindingsImport);
			InputBindingSettingsSection->OnResetDefaults().BindRaw(this, &FInputBindingEditorModule::HandleInputBindingsResetToDefault);
			InputBindingSettingsSection->OnSave().BindRaw(this, &FInputBindingEditorModule::HandleInputBindingsSave);
		}
	}

	virtual void ShutdownModule() override
	{
		if(FModuleManager::Get().IsModuleLoaded(InputBindingEditorModule::PropertyEditorModuleName))
		{
			FPropertyEditorModule& PropertyEditor = FModuleManager::GetModuleChecked<FPropertyEditorModule>(InputBindingEditorModule::PropertyEditorModuleName);

			PropertyEditor.UnregisterCustomClassLayout(EditorKeyboardShortcutSettingsName);
		}

	}

private:
	// Show a warning that the editor will require a restart and return its result
	EAppReturnType::Type ShowRestartWarning(const FText& Title) const
	{
		return FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("ActionRestartMsg", "Imported settings won't be applied until the editor is restarted. Do you wish to restart now (you will be prompted to save any changes)?"), Title);
	}

	// Backup a file
	bool BackupFile(const FString& SrcFilename, const FString& DstFilename)
	{
		if(IFileManager::Get().Copy(*DstFilename, *SrcFilename) == COPY_OK)
		{
			return true;
		}

		// log error	
		FMessageLog EditorErrors("EditorErrors");
		if(!FPaths::FileExists(SrcFilename))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("FileName"), FText::FromString(SrcFilename));
			EditorErrors.Warning(FText::Format(LOCTEXT("UnsuccessfulBackup_NoExist_Notification", "Unsuccessful backup! {FileName} does not exist!"), Arguments));
		}
		else if(IFileManager::Get().IsReadOnly(*DstFilename))
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("FileName"), FText::FromString(DstFilename));
			EditorErrors.Warning(FText::Format(LOCTEXT("UnsuccessfulBackup_ReadOnly_Notification", "Unsuccessful backup! {FileName} is read-only!"), Arguments));
		}
		else
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("SourceFileName"), FText::FromString(SrcFilename));
			Arguments.Add(TEXT("BackupFileName"), FText::FromString(DstFilename));
			// We don't specifically know why it failed, this is a fallback.
			EditorErrors.Warning(FText::Format(LOCTEXT("UnsuccessfulBackup_Fallback_Notification", "Unsuccessful backup of {SourceFileName} to {BackupFileName}"), Arguments));
		}
		EditorErrors.Notify(LOCTEXT("BackupUnsuccessful_Title", "Backup Unsuccessful!"));

		return false;
	}


	// Handles exporting input bindings to a file
	bool HandleInputBindingsExport(const FString& Filename)
	{
		FInputBindingManager::Get().SaveInputBindings();
		GConfig->Flush(false, GEditorKeyBindingsIni);

		// If the user has no user-defined chords, the file won't be saved
		if (!IFileManager::Get().FileExists(*GEditorKeyBindingsIni))
		{
			return false;
		}

		return BackupFile(GEditorKeyBindingsIni, Filename);
	}

	// Handles importing input bindings from a file
	bool HandleInputBindingsImport(const FString& Filename)
	{
		if (!IFileManager::Get().FileExists(*Filename))
		{
			return false;
		}

		FConfigFile TempConfig;
		TempConfig.Read(Filename);

		// Don't bother prompting for a restart if the file doesn't actually contain any user-defined chords
		if (!TempConfig.Contains(TEXT("UserDefinedChords")))
		{
			return false;
		}

		if(EAppReturnType::Ok == ShowRestartWarning(LOCTEXT("ImportKeyBindings_Title", "Import Key Bindings")))
		{
			FUnrealEdMisc::Get().SetConfigRestoreFilename(Filename, GEditorKeyBindingsIni);
			FUnrealEdMisc::Get().RestartEditor(false);

			return true;
		}

		return false;
	}

	// Handles resetting input bindings back to the defaults
	bool HandleInputBindingsResetToDefault()
	{
		if(EAppReturnType::Ok == ShowRestartWarning(LOCTEXT("ResetKeyBindings_Title", "Reset Key Bindings")))
		{
			FInputBindingManager::Get().RemoveUserDefinedChords();
			GConfig->Flush(false, GEditorKeyBindingsIni);
			FUnrealEdMisc::Get().RestartEditor(false);

			return true;
		}

		return false;
	}

	// Handles saving default input bindings.
	bool HandleInputBindingsSave()
	{
		FInputBindingManager::Get().RemoveUserDefinedChords();
		GConfig->Flush(false, GEditorKeyBindingsIni);
		return true;
	}
private:

	/** Holds the collection of created binding editor panels. */
	TArray<TSharedPtr<SWidget> > BindingEditorPanels;

	/** Captured name of the UEditorKeyboardShortcutSettings class */
	FName EditorKeyboardShortcutSettingsName;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FInputBindingEditorModule, InputBindingEditor);
