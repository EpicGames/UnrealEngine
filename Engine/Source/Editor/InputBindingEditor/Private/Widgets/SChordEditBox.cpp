// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChordEditBox.h"

#include "Framework/Application/SlateApplication.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleColors.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SChordEditor.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;


#define LOCTEXT_NAMESPACE "SChordEditBox"


/* SChordEditBox interface
 *****************************************************************************/

void SChordEditBox::Construct(const FArguments& InArgs, TSharedPtr<FUICommandInfo> InputCommand, EMultipleKeyBindingIndex InChordIndex)
{
	// Match EditableTextBox style
	const FEditableTextBoxStyle& EditableTextBoxStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	BorderImageNormal = &EditableTextBoxStyle.BackgroundImageNormal;
	BorderImageHovered = &EditableTextBoxStyle.BackgroundImageHovered;
	BorderImageFocused = &EditableTextBoxStyle.BackgroundImageFocused;

	ChildSlot
	[
		SAssignNew(ConflictPopup, SMenuAnchor)
		.Placement(MenuPlacement_ComboBox)
		.OnGetMenuContent(this, &SChordEditBox::OnGetContentForConflictPopup)
		.OnMenuOpenChanged(this, &SChordEditBox::OnConflictPopupOpenChanged)
		[
			SNew(SBorder)
			.VAlign(VAlign_Center)
			.Padding(0)
			.BorderImage(this, &SChordEditBox::GetBorderImage)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.HAlign(HAlign_Fill)
				.FillWidth(1)
				[
					SNew(SBox)
					.Padding(EditableTextBoxStyle.Padding)
					.VAlign(VAlign_Center)
					.WidthOverride(200.f)
					[
						SAssignNew(ChordEditor, SChordEditor, InputCommand, InChordIndex)
						.OnEditBoxLostFocus(this, &SChordEditBox::OnChordEditorLostFocus)
						.OnChordChanged(this, &SChordEditBox::OnChordChanged)
						.OnEditingStarted(this, &SChordEditBox::OnChordEditingStarted)
						.OnEditingStopped(this, &SChordEditBox::OnChordEditingStopped)
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					// Remove binding button
					SNew(SButton)
					.Visibility(this, &SChordEditBox::GetChordRemoveButtonVisibility)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.ContentPadding(0.f)
					.OnClicked(this, &SChordEditBox::OnChordRemoveButtonClicked)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					.ToolTipText(LOCTEXT("ChordEditButtonRemove_ToolTip", "Remove this binding"))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.X"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
	];
}


const FSlateBrush* SChordEditBox::GetBorderImage() const
{
	if ( ChordEditor->HasKeyboardFocus() )
	{
		return BorderImageFocused;
	}
	else
	{
		if ( ChordEditor->IsHovered() )
		{
			return BorderImageHovered;
		}
		else
		{
			return BorderImageNormal;
		}
	}
}


FText SChordEditBox::GetNotificationMessage() const
{
	return ChordEditor->GetNotificationText();
}


void SChordEditBox::OnChordEditorLostFocus()
{
	if( (!ChordAcceptButton.IsValid() || ChordAcceptButton->HasMouseCapture() == false) && !ChordEditor->IsTyping() && !ChordEditor->HasConflict() )
	{
		if( ChordEditor->IsEditing() && ChordEditor->IsEditedChordValid() )
		{
			ChordEditor->CommitNewChord();
		}

		ChordEditor->StopEditing();
	}
}


void SChordEditBox::OnChordEditingStarted()
{
	ConflictPopup->SetIsOpen(false);
}


void SChordEditBox::OnChordEditingStopped()
{
	if( ChordEditor->IsEditedChordValid() && !ChordEditor->HasConflict() )
	{
		ChordEditor->CommitNewChord();
	}
}


void SChordEditBox::OnChordChanged()
{
	if( ChordEditor->HasConflict() )
	{
		ConflictPopup->SetIsOpen(true, true);
	}
	else
	{
		ConflictPopup->SetIsOpen(false);

		if (ChordEditor->IsEditedChordValid())
		{
			ChordEditor->CommitNewChord();
			ChordEditor->StopEditing();
			FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::SetDirectly);
		}
	}
}

EVisibility SChordEditBox::GetChordRemoveButtonVisibility() const
{
	if( !ChordEditor->IsEditing() && ChordEditor->IsActiveChordValid() )
	{
		// Chord should display a button.  What the button does is defined by the two cases above
		return EVisibility::Visible;
	}
	else
	{
		// Nothing to remove, but still take up the space
		return EVisibility::Hidden;
	}
}


FReply SChordEditBox::OnChordRemoveButtonClicked()
{
	if( !ChordEditor->IsEditing() )
	{
		ChordEditor->RemoveActiveChord();
	}

	return FReply::Handled();
}


FReply SChordEditBox::OnAcceptNewChordButtonClicked()
{
	if( ChordEditor->IsEditing() )
	{
		ChordEditor->CommitNewChord();
		ChordEditor->StopEditing();
		
	}

	ConflictPopup->SetIsOpen(false);

	return FReply::Handled();
}


TSharedRef<SWidget> SChordEditBox::OnGetContentForConflictPopup()
{
	return SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("NotificationList.ItemBackground")  )
		[
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding( 2.0f, 0.0f )
				.AutoHeight()
				[
					SNew( STextBlock )
						.WrapTextAt(200.0f)
						.ColorAndOpacity( FStyleColors::Warning )
						.Text( this, &SChordEditBox::GetNotificationMessage )
						.Visibility( this, &SChordEditBox::GetNotificationVisibility )
				]

			+ SVerticalBox::Slot()
				.Padding( 2.0f )
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoHeight()
				[
					SAssignNew( ChordAcceptButton, SButton )
						.ContentPadding(1)
						.ToolTipText( LOCTEXT("ChordAcceptButton_ToolTip", "Accept this new binding") )
						.OnClicked( this, &SChordEditBox::OnAcceptNewChordButtonClicked )
						[
							SNew( SHorizontalBox )

							+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew( SImage )
										.Image( FAppStyle::GetBrush( "Symbols.Check" ) )
										.ColorAndOpacity( FLinearColor(0,.7f,0,.75f) )
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(2.0f,0.0f)
								[
									SNew( STextBlock )
										.Text( LOCTEXT("ChordAcceptButtonText_Override", "Override") )
								]
						]
				]
		];
}

void SChordEditBox::OnConflictPopupOpenChanged(bool bIsOpen)
{
	if(!bIsOpen)
	{
		ChordEditor->StopEditing();
	}
}

EVisibility SChordEditBox::GetNotificationVisibility() const
{
	return !ChordEditor->GetNotificationText().IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed;
}


FReply SChordEditBox::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent )
{
	// This is a passthrough if the chord edit box gets a mouse click in the button area and the button isn't visible. 
	// we should focus the lower level editing widget in this case
	if ( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		if( !ChordEditor->IsEditing() )
		{
			ChordEditor->StartEditing();
		}
		return FReply::Handled().SetUserFocus(ChordEditor.ToSharedRef(), EFocusCause::Mouse);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
