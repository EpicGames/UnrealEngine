// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandInfo.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SButton;
class SMenuAnchor;
class SWidget;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

/**
 * A widget that adds visuals to an SChordEditor
 */
class SChordEditBox
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SChordEditBox ){}
	SLATE_END_ARGS()
public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 */
	void Construct( const FArguments& InArgs, TSharedPtr<FUICommandInfo> InputCommand, EMultipleKeyBindingIndex ChordIndex );
private:

	/** @return Border image for the text box based on the hovered and focused state */
	const FSlateBrush* GetBorderImage() const;

	FText GetNotificationMessage() const;

	/** Called when the chord editor loses focus */
	void OnChordEditorLostFocus();

	/** Called when editing starts in the chord editor */
	void OnChordEditingStarted();

	/** Called when editing stops in the chord editor */
	void OnChordEditingStopped();

	/** Called when the edited chord changes */
	void OnChordChanged();

	/** @return the visibility of the chord edit button */
	EVisibility GetChordRemoveButtonVisibility() const;

	/** Called when the chord edit button is clicked */
	FReply OnChordRemoveButtonClicked();

	/** Called when the accept button is clicked.  */
	FReply OnAcceptNewChordButtonClicked();

	/** @return content to be shown in the key binding conflict pop-up */
	TSharedRef<SWidget> OnGetContentForConflictPopup();

	/** Called when conflict popup opens or closes */
	void OnConflictPopupOpenChanged(bool bIsOpen);

	/** @return The visibility of the duplicate binding notification area */
	EVisibility GetNotificationVisibility() const; 
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent );

private:

	/** The chord editor for this box */
	TSharedPtr<class SChordEditor> ChordEditor;
	
	/** Menu anchor where the conflict pop-up is shown */
	TSharedPtr<SMenuAnchor> ConflictPopup;
	
	/** The button for committing chord */
	mutable TSharedPtr<SButton> ChordAcceptButton;
	
	/** Styling: border image to draw when not hovered or focused */
	const FSlateBrush* BorderImageNormal;
	
	/** Styling: border image to draw when hovered */
	const FSlateBrush* BorderImageHovered;
	
	/** Styling: border image to draw when focused */
	const FSlateBrush* BorderImageFocused;
};
