// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChordEditor.h"

#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/InputBindingManager.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Misc/AssertionMacros.h"
#include "Styling/CoreStyle.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "Widgets/Text/SlateEditableTextLayout.h"

struct FGeometry;

#define LOCTEXT_NAMESPACE "SChordEditor"


/* SChordEditor interface
 *****************************************************************************/

void SChordEditor::Construct( const FArguments& InArgs, TSharedPtr<FUICommandInfo> InputCommand, EMultipleKeyBindingIndex InChordIndex)
{
	bIsEditing = false;

	CommandInfo = InputCommand;
	ChordIndex = InChordIndex;
	OnEditBoxLostFocus = InArgs._OnEditBoxLostFocus;
	OnChordChanged = InArgs._OnChordChanged;
	OnEditingStopped = InArgs._OnEditingStopped;
	OnEditingStarted = InArgs._OnEditingStarted;

	SEditableText::Construct( 
		SEditableText::FArguments() 
		.Text( this, &SChordEditor::OnGetChordInputText )
		.HintText( this, &SChordEditor::OnGetChordInputHintText )
		.Font( FCoreStyle::GetDefaultFontStyle("Regular", 9) )
		.HintTextOpacity(0.25f)
	);

	SlateEditableTextLayout.LoadText();
}


TWeakPtr<SChordEditor> SChordEditor::ChordBeingEdited;


FReply SChordEditor::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	const FKey Key = InKeyEvent.GetKey();

	if( bIsEditing ) 
	{

		TGuardValue<bool> TypingGuard( bIsTyping, true );

		if( !EKeys::IsModifierKey(Key) )
		{
			EditingInputChord.Key = Key;
		}
	
		SlateEditableTextLayout.BeginEditTransaction();

		EditingInputChord.bCtrl = InKeyEvent.IsControlDown();
		EditingInputChord.bAlt = InKeyEvent.IsAltDown();
		EditingInputChord.bShift = InKeyEvent.IsShiftDown();
		EditingInputChord.bCmd = InKeyEvent.IsCommandDown();

		SlateEditableTextLayout.LoadText();
		SlateEditableTextLayout.GoTo(ETextLocation::EndOfDocument);

		SlateEditableTextLayout.EndEditTransaction();
		
		OnChordTyped( EditingInputChord );
		
		return FReply::Handled();
	}

	return FReply::Unhandled();
}


FReply SChordEditor::OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	return FReply::Unhandled();
}


FReply SChordEditor::OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent )
{
	return FReply::Unhandled();
}


FReply SChordEditor::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent )
{
	if ( InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && !bIsEditing )
	{
		StartEditing();
		return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::Mouse);
	}

	return FReply::Unhandled();
}


FReply SChordEditor::OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	return FReply::Handled();
}


void SChordEditor::OnFocusLost( const FFocusEvent& InFocusEvent )
{
	SEditableText::OnFocusLost(InFocusEvent);
	// Notify a listener that we lost focus so they can determine if we should still be in edit mode
	OnEditBoxLostFocus.ExecuteIfBound();
}


FText SChordEditor::OnGetChordInputText() const
{
	if( bIsEditing )
	{
		return EditingInputChord.GetInputText();
	}
	else if( CommandInfo->GetActiveChord(ChordIndex)->IsValidChord() )
	{
		return CommandInfo->GetActiveChord(ChordIndex)->GetInputText();
	}
	else
	{
		return FText::GetEmpty();
	}
}


FText SChordEditor::OnGetChordInputHintText() const
{
	const TSharedPtr<const FInputChord>& ActiveChord = CommandInfo->GetActiveChord(ChordIndex);
	
	if( !bIsEditing || !CommandInfo->GetDefaultChord(ChordIndex).IsValidChord() )
	{
		return LOCTEXT("NewBindingHelpText_NoCurrentBinding", "Type a new binding");

	}
	else if( bIsEditing )
	{
		FFormatNamedArguments Args;
		Args.Add( TEXT("InputCommandBinding"), CommandInfo->GetDefaultChord(ChordIndex).GetInputText() );
		return FText::Format( LOCTEXT("NewBindingHelpText_CurrentBinding", "Default: {InputCommandBinding}"), Args );
	}
	
	return FText::GetEmpty();
}


void SChordEditor::CommitNewChord()
{
	if( EditingInputChord.IsValidChord() )
	{
		OnChordCommitted( EditingInputChord);
	}
}


void SChordEditor::RemoveActiveChord()
{
	CommandInfo->RemoveActiveChord(ChordIndex);
}


void SChordEditor::StartEditing()
{
	if( ChordBeingEdited.IsValid()  )
	{
		ChordBeingEdited.Pin()->StopEditing();
	}

	ChordBeingEdited = SharedThis( this );

	NotificationMessage = FText::GetEmpty();
	EditingInputChord = FInputChord( EKeys::Invalid, EModifierKey::None );
	bIsEditing = true;

	OnEditingStarted.ExecuteIfBound();
}


void SChordEditor::StopEditing()
{
	if( ChordBeingEdited.IsValid() && ChordBeingEdited.Pin().Get() == this )
	{
		ChordBeingEdited.Reset();
	}

	OnEditingStopped.ExecuteIfBound();

	bIsEditing = false;

	EditingInputChord = FInputChord( EKeys::Invalid, EModifierKey::None );
	NotificationMessage = FText::GetEmpty();
}


void SChordEditor::OnChordTyped( const FInputChord& NewChord )
{
	// Check to see if a valid sequence was entered
	if( NewChord.IsValidChord() )
	{
		// The chord doesn't exist in the command so now we need to make sure the chord doesn't exist in another command in the same context
		FName ContextName = CommandInfo->GetBindingContext();

		const bool bCheckDefaultChord = false;
		const TSharedPtr<FUICommandInfo> FoundDesc = FInputBindingManager::Get().GetCommandInfoFromInputChord( ContextName, NewChord, bCheckDefaultChord );

		if( FoundDesc.IsValid() && FoundDesc->GetCommandName() != CommandInfo->GetCommandName() )
		{
			// Chord already exists
			NotificationMessage = FText::Format( LOCTEXT("KeyAlreadyBound", "{0} is already bound to {1}"), NewChord.GetInputText(), FoundDesc->GetLabel() );
		}
		else
		{

			NotificationMessage = FText::GetEmpty();
		}
	}

	OnChordChanged.ExecuteIfBound();
}


void SChordEditor::OnChordCommitted( const FInputChord& NewChord)
{
	// This delegate is only called on valid chords
	check( NewChord.IsValidChord() );

	{
		const FName ContextName = CommandInfo->GetBindingContext();

		const bool bCheckDefaultChord = false;
		const TSharedPtr<FUICommandInfo> FoundDesc = FInputBindingManager::Get().GetCommandInfoFromInputChord( ContextName, NewChord, bCheckDefaultChord );

		if( FoundDesc.IsValid() && FoundDesc->GetCommandName() != CommandInfo->GetCommandName() )
		{
			// Remove the active chord on the command that was already bound to the chord being set on another command.
			for (uint32 i = 0; i < static_cast<uint8>(EMultipleKeyBindingIndex::NumChords); ++i)
			{
				EMultipleKeyBindingIndex RemovableIndex = static_cast<EMultipleKeyBindingIndex> (i);
				if (*FoundDesc->GetActiveChord(RemovableIndex) == NewChord)
				{
					FoundDesc->RemoveActiveChord(RemovableIndex);
				}
			}
		}

		// Set the new chord on the command being edited
		CommandInfo->SetActiveChord( NewChord, ChordIndex);
	}
}


#undef LOCTEXT_NAMESPACE
