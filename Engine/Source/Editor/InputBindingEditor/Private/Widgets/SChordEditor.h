// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SEditableText.h"

struct FCharacterEvent;
struct FFocusEvent;
struct FGeometry;
struct FKeyEvent;
struct FPointerEvent;

/**
 * A specialized text edit box that visualizes a new chord being entered           .
 */
class SChordEditor
	: public SEditableText
{
public:

	SLATE_BEGIN_ARGS( SChordEditor ){}
		SLATE_EVENT( FSimpleDelegate, OnEditBoxLostFocus )
		SLATE_EVENT( FSimpleDelegate, OnChordChanged )
		SLATE_EVENT( FSimpleDelegate, OnEditingStopped )
		SLATE_EVENT( FSimpleDelegate, OnEditingStarted )
	SLATE_END_ARGS()

public:

	/** Default constructor. */
	SChordEditor()
		: bIsEditing( false )
		, bIsTyping( false )
	{ }

public:

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The Slate argument list.
	 * @param InputCommand
	 */
	void Construct( const FArguments& InArgs, TSharedPtr<FUICommandInfo> InputCommand, EMultipleKeyBindingIndex ChordIndex);
	
	virtual bool SupportsKeyboardFocus() const override { return true; }

	/** Starts editing the chord. */
	void StartEditing();
	
	/** Stops editing the chord. */
	void StopEditing();

	/** Commits the new chord to the commands active chord. */
	void CommitNewChord();

	/** Removes the active chord from the command. */
	void RemoveActiveChord();

	/** @return Whether or not we are in editing mode. */
	bool IsEditing() const { return bIsEditing; }

	/** @return True if the user is physically typing a key. */
	bool IsTyping() const { return bIsTyping; }

	/** @return Whether or not the chord being edited is valid. */
	bool IsEditedChordValid() const { return EditingInputChord.IsValidChord(); }

	/** @return Whether or not the command has a valid chord. */
	bool IsActiveChordValid() const { return CommandInfo->GetActiveChord(ChordIndex)->IsValidChord(); }

	/** @return the Notification message being displayed if any. */
	const FText& GetNotificationText() const { return NotificationMessage; };

	/** @return true if the edited chord has a conflict with an existing chord. */
	bool HasConflict() const { return !NotificationMessage.IsEmpty(); }

private:

	/** Chord that is currently being edited. */
	static TWeakPtr<SChordEditor> ChordBeingEdited;

	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnKeyUp( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& InMouseEvent ) override;
	virtual FReply OnMouseButtonDoubleClick( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent ) override;
	virtual void OnFocusLost( const FFocusEvent& InFocusEvent ) override;
	virtual FReply OnKeyChar( const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent ) override;

	/** 
	 * Called when the chord changes.
	 *
	 * @param NewChord The new chord.
	 */
	void OnChordTyped( const FInputChord& NewChord );

	/** 
	 * Called when the chord changes.
	 *
	 * @param NewChord The chord to commit.
	 */
	void OnChordCommitted( const FInputChord& NewChord );

	/** @return The chord input text (If editing, the edited chord, otherwise the active chord) */
	FText OnGetChordInputText() const;
	
	/** @return The hint text to display in the text box if it is empty */
	FText OnGetChordInputHintText() const;

private:

	/** The command we are editing a chord for. */
	TSharedPtr<FUICommandInfo> CommandInfo;

	/** The index of the chord we are editing (within the multiple key bindings. */
	EMultipleKeyBindingIndex ChordIndex;

	/** Delegate to execute when the edit box loses focus. */
	FSimpleDelegate OnEditBoxLostFocus;

	/** Delegate to execute when the chord changes. */
	FSimpleDelegate OnChordChanged;

	/** Delegate to execute when we stop editing. */
	FSimpleDelegate OnEditingStopped;

	/** Delegate to execute when we start editing. */
	FSimpleDelegate OnEditingStarted;

	/** The notification message (duplicate bindings) being displayed. */
	FText NotificationMessage;

	/** Temp chord being edited. */
	FInputChord EditingInputChord;

	/** Whether or not we are in edit mode. */
	bool bIsEditing;

	/** Whether or not the user is physically typing a new key. */
	bool bIsTyping;
};
