// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for input binding editor modules.
 */
class IInputBindingEditorModule
	: public IModuleInterface
{
public:

	/** Virtual destructor. */
	virtual ~IInputBindingEditorModule( ) = default;
};
